{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Blueprint Stats",
	"Description" : "Blueprint Stats",
	"Category" : "Blueprints",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : false,
	"Installed" : false,
	"Modules" :
	[
		{
			"Name" : "BlueprintStats",
			"Type" : "Editor",
			"LoadingPhase" : "Default"
		}
	]
}