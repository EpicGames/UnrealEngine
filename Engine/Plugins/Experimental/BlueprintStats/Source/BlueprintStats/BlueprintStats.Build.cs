// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class BlueprintStats : ModuleRules
	{
        public BlueprintStats(ReadOnlyTargetRules Target) : base(Target)
		{
			PublicDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"EditorFramework",
					"UnrealEd",
					"GraphEditor",
					"BlueprintGraph",
					"MessageLog"
				}
			);
		}
	}
}
