// Copyright Epic Games, Inc. All Rights Reserved.

#include "BlueprintStats.h"
#include "Kismet2/BlueprintEditorUtils.h"
#include "K2Node_CallFunction.h"
#include "K2Node_MacroInstance.h"

#define LOCTEXT_NAMESPACE "BlueprintStats"

UFunction* GetSupererestFunction(UFunction* TestFunction)
{
	UFunction* MostAncestral = NULL;
	while (TestFunction != NULL)
	{
		MostAncestral = TestFunction;
		TestFunction = TestFunction->GetSuperFunction();
	}

	return MostAncestral;
}

void FBlueprintStatRecord::ReadStatsFromBlueprint()
{
	check(SourceBlueprint != NULL);

	++NumBlueprints;
	if (FBlueprintEditorUtils::IsDataOnlyBlueprint(SourceBlueprint))
	{
		++NumDataOnlyBlueprints;
	}

	NumUserMacros += SourceBlueprint->MacroGraphs.Num();
	for (const UEdGraph* FunctionGraph : SourceBlueprint->FunctionGraphs)
	{
		if (UFunction* Function = FindUField<UFunction>(SourceBlueprint->GeneratedClass, FunctionGraph->GetFName()))
		{
			// Make sure we've got the native decl if it was an override
			Function = GetSupererestFunction(Function);

			if (Function->GetOwnerClass() == SourceBlueprint->GeneratedClass)
			{
				// User-defined function, introduced in this class
				++NumUserFunctions;

				if (Function->HasAnyFunctionFlags(FUNC_BlueprintPure))
				{
					++NumUserPureFunctions;
				}
			}
			else
			{
				//@TODO: gather stats about overrides too
			}
		}
	}

	TArray<UEdGraphNode*> Nodes;
	FBlueprintEditorUtils::GetAllNodesOfClass(SourceBlueprint, Nodes);

	const UEdGraphSchema_K2* K2Schema = GetDefault<UEdGraphSchema_K2>();

	for (const UEdGraphNode* Node : Nodes)
	{
		++NumNodes;

		// See what kinds of pins this node has
		int32 DataInputs = 0;
		int32 DataOutputs = 0;
		int32 ExecInputs = 0;
		int32 ExecOutputs = 0;
		int32 NumSelfPins = 0;
		for (const UEdGraphPin* Pin : Node->Pins)
		{
			if (Pin->bHidden)
			{
				continue;
			}

			if (K2Schema->IsExecPin(*Pin))
			{
				ExecInputs += (Pin->Direction == EGPD_Input) ? 1 : 0;
				ExecOutputs += (Pin->Direction == EGPD_Output) ? 1 : 0;
			}
			else
			{
				if (K2Schema->IsSelfPin(*Pin))
				{
					++NumSelfPins;
				}

				DataInputs += (Pin->Direction == EGPD_Input) ? 1 : 0;
				DataOutputs += (Pin->Direction == EGPD_Output) ? 1 : 0;
			}
		}

		// Generic stuff
		NodeCount.FindOrAdd(Node->GetClass()) += 1;

		// 
		if (const UK2Node_CallFunction* FunctionNode = Cast<const UK2Node_CallFunction>(Node))
		{
			if (UFunction* TargetFunction = FunctionNode->GetTargetFunction())
			{
				FunctionCount.FindOrAdd(TargetFunction) += 1;
				FunctionOwnerCount.FindOrAdd(TargetFunction->GetOwnerClass()) += 1;
			}

			if (FunctionNode->IsNodePure())
			{
				++PureFunctionNodes;
			}
			else
			{
				++ImpureFunctionNodes;
				if (DataInputs > 0)
				{
					++ImpureNodesWithInputs;
				}

				if (DataOutputs > 0)
				{
					++ImpureNodesWithOutputs;
				}

				if ((DataInputs > NumSelfPins) && (DataOutputs > 0))
				{
					++ImpureNodesWithInputsAndOutputs;
				}
			}
		}
		else if (const UK2Node_MacroInstance* MacroNode = Cast<const UK2Node_MacroInstance>(Node))
		{
			if (UEdGraph* MacroGraph = MacroNode->GetMacroGraph())
			{
				if (MacroGraph != NULL)
				{
					if (FBlueprintEditorUtils::FindBlueprintForGraph(MacroGraph) != SourceBlueprint)
					{
						RemoteMacroCount.FindOrAdd(MacroGraph) += 1;
					}
				}
			}
		}
	}
}

void FBlueprintStatRecord::MergeAnotherRecordIn(const FBlueprintStatRecord& OtherRecord)
{
	SourceBlueprint = NULL;

	for (const auto& NodeCountPair : OtherRecord.NodeCount)
	{
		NodeCount.FindOrAdd(NodeCountPair.Key) += NodeCountPair.Value;
	}

	for (const auto& FunctionCountPair : OtherRecord.FunctionCount)
	{
		FunctionCount.FindOrAdd(FunctionCountPair.Key) += FunctionCountPair.Value;
	}

	for (const auto& FunctionOwnerCountPair : OtherRecord.FunctionOwnerCount)
	{
		FunctionOwnerCount.FindOrAdd(FunctionOwnerCountPair.Key) += FunctionOwnerCountPair.Value;
	}

	for (const auto& RemoteMacroCountPair : OtherRecord.RemoteMacroCount)
	{
		RemoteMacroCount.FindOrAdd(RemoteMacroCountPair.Key) += RemoteMacroCountPair.Value;
	}

	ImpureNodesWithInputs += OtherRecord.ImpureNodesWithInputs;
	ImpureNodesWithOutputs += OtherRecord.ImpureNodesWithOutputs;
	ImpureNodesWithInputsAndOutputs += OtherRecord.ImpureNodesWithInputsAndOutputs;
	ImpureFunctionNodes += OtherRecord.ImpureFunctionNodes;
	PureFunctionNodes += OtherRecord.PureFunctionNodes;
	NumUserFunctions += OtherRecord.NumUserFunctions;
	NumUserPureFunctions += OtherRecord.NumUserPureFunctions;
	NumUserMacros += OtherRecord.NumUserMacros;
	NumBlueprints += OtherRecord.NumBlueprints;
	NumDataOnlyBlueprints += OtherRecord.NumDataOnlyBlueprints;
	NumNodes += OtherRecord.NumNodes;
}

FString FBlueprintStatRecord::ToString(bool bHeader) const
{
	if (bHeader)
	{
		return TEXT("Total,DOBP,NumNodes,ImpureWI,ImpureWO,ImpureWIO,ImpureTotal,PureTotal,UserFnCount,UserPureCount,UserMacroCount");
	}
	else
	{
		return FString::Printf(TEXT("%d,%d,%d,%d,%d,%d,%d,%d,%d,%d,%d"),
			NumBlueprints,
			NumDataOnlyBlueprints,
			NumNodes,
			ImpureNodesWithInputs,
			ImpureNodesWithOutputs,
			ImpureNodesWithInputsAndOutputs,
			ImpureFunctionNodes,
			PureFunctionNodes,
			NumUserFunctions,
			NumUserPureFunctions,
			NumUserMacros
			);
	}
}


#undef LOCTEXT_NAMESPACE
