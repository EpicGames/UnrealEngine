// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Map.h"

class UFunction;

class UBlueprint;
class UEdGraph;

class FBlueprintStatRecord
{
public:
	// Can be NULL, if it's a meta-record
	UBlueprint* SourceBlueprint;

	TMap<UClass*, int32> NodeCount;
	TMap<UFunction*, int32> FunctionCount;
	TMap<UClass*, int32> FunctionOwnerCount;
	TMap<UEdGraph*, int32> RemoteMacroCount;

	int32 ImpureNodesWithInputs;
	int32 ImpureNodesWithOutputs;
	int32 ImpureNodesWithInputsAndOutputs;
	int32 ImpureFunctionNodes;
	int32 PureFunctionNodes;
	int32 NumUserFunctions;  // pure or impure
	int32 NumUserPureFunctions;
	int32 NumUserMacros;
	int32 NumBlueprints;
	int32 NumDataOnlyBlueprints;
	int32 NumNodes;
public:
	FBlueprintStatRecord(UBlueprint* InBlueprint = NULL)
		: SourceBlueprint(InBlueprint)
		, ImpureNodesWithInputs(0)
		, ImpureNodesWithOutputs(0)
		, ImpureNodesWithInputsAndOutputs(0)
		, ImpureFunctionNodes(0)
		, PureFunctionNodes(0)
		, NumUserFunctions(0)
		, NumUserPureFunctions(0)
		, NumUserMacros(0)
		, NumBlueprints(0)
		, NumDataOnlyBlueprints(0)
		, NumNodes(0)
	{
		if (SourceBlueprint != NULL)
		{
			ReadStatsFromBlueprint();
		}
	}

	void MergeAnotherRecordIn(const FBlueprintStatRecord& OtherRecord);

	FString ToString(bool bHeader = false) const;
protected:
	void ReadStatsFromBlueprint();
};
