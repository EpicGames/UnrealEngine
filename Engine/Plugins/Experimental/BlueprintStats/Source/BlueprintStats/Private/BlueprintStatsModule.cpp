// Copyright Epic Games, Inc. All Rights Reserved.

#include "Templates/Greater.h"
#include "HAL/IConsoleManager.h"
#include "Misc/App.h"
#include "UObject/UObjectIterator.h"
#include "Engine/Blueprint.h"
#include "EdGraph/EdGraph.h"
#include "IBlueprintStatsModule.h"
#include "BlueprintStats.h"

DEFINE_LOG_CATEGORY_STATIC(LogBlueprintStats, Verbose, All);

class FBlueprintStatsModule : public IBlueprintStatsModule
{
public:
	FBlueprintStatsModule()
		: DumpBlueprintStatsCmd(NULL)
	{
	}

	// IModuleInterface interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End of IModuleInterface interface

private:
	IConsoleObject* DumpBlueprintStatsCmd;

private:
	static void DumpBlueprintStats();
};

IMPLEMENT_MODULE(FBlueprintStatsModule, BlueprintStats)


void FBlueprintStatsModule::StartupModule()
{
	if (!IsRunningCommandlet())
	{
		DumpBlueprintStatsCmd = IConsoleManager::Get().RegisterConsoleCommand(
			TEXT("DumpBlueprintStats"),
			TEXT("Dumps statistics about blueprint node usage to the log."),
			FConsoleCommandDelegate::CreateStatic(DumpBlueprintStats),
			ECVF_Default
			);
	}
}

void FBlueprintStatsModule::ShutdownModule()
{
	if (DumpBlueprintStatsCmd != NULL)
	{
		IConsoleManager::Get().UnregisterConsoleObject(DumpBlueprintStatsCmd);
	}
}

void FBlueprintStatsModule::DumpBlueprintStats()
{
	TArray<FBlueprintStatRecord> Records;
	for (TObjectIterator<UBlueprint> BlueprintIt; BlueprintIt; ++BlueprintIt)
	{
		UBlueprint* Blueprint = *BlueprintIt;

		new (Records) FBlueprintStatRecord(Blueprint);
	}


	// Now merge them
	FBlueprintStatRecord Aggregate(NULL);
	for (const FBlueprintStatRecord& SourceRecord : Records) //-V1078
	{
		Aggregate.MergeAnotherRecordIn(SourceRecord);
	}

	// Sort the lists
	Aggregate.NodeCount.ValueSort(TGreater<int32>());
	Aggregate.FunctionCount.ValueSort(TGreater<int32>());
	Aggregate.FunctionOwnerCount.ValueSort(TGreater<int32>());
	Aggregate.RemoteMacroCount.ValueSort(TGreater<int32>());

	// Print out the merged record
	UE_LOGF(LogBlueprintStats, Log, "Blueprint stats for %d blueprints in %ls", Records.Num(), FApp::GetProjectName());
	UE_LOGF(LogBlueprintStats, Log, "%ls", *Aggregate.ToString(true));
	UE_LOGF(LogBlueprintStats, Log, "%ls", *Aggregate.ToString(false));
	UE_LOGF(LogBlueprintStats, Log, "\n");

	// Print out the node list
	UE_LOGF(LogBlueprintStats, Log, "NodeClass,NumInstances");
	for (const auto& NodePair : Aggregate.NodeCount)
	{
		UE_LOGF(LogBlueprintStats, Log, "%ls,%d", *(NodePair.Key->GetName()), NodePair.Value);
	}
	UE_LOGF(LogBlueprintStats, Log, "\n");

	// Print out the function list
	UE_LOGF(LogBlueprintStats, Log, "FunctionPath,ClassName,FunctionName,NumInstances");
	for (const auto& FunctionPair : Aggregate.FunctionCount)
	{
		UFunction* Function = FunctionPair.Key;
		UE_LOGF(LogBlueprintStats, Log, "%ls,%ls,%ls,%d", *(Function->GetPathName()), *(Function->GetOuterUClass()->GetName()), *(Function->GetName()), FunctionPair.Value);
	}
	UE_LOGF(LogBlueprintStats, Log, "\n");

	// Print out the macro list
	UE_LOGF(LogBlueprintStats, Log, "MacroPath,MacroName,NumInstances");
	for (const auto& MacroPair : Aggregate.RemoteMacroCount)
	{
		UEdGraph* MacroGraph = MacroPair.Key;
		UE_LOGF(LogBlueprintStats, Log, "%ls,%ls,%d", *(MacroGraph->GetPathName()), *(MacroGraph->GetName()), MacroPair.Value);
	}
	UE_LOGF(LogBlueprintStats, Log, "\n");
}
